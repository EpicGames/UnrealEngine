// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateNullRenderer.h"
#include "Rendering/DrawElements.h"
#include "Rendering/ShaderResourceManager.h"
#include "Rendering/SlateDrawBuffer.h"
#if UE_SLATE_NULL_RENDERER_WITH_ENGINE
#include "RenderingThread.h"
#endif


FSlateNullRenderer::FSlateNullRenderer(const TSharedRef<FSlateFontServices>& InSlateFontServices, const TSharedRef<FSlateShaderResourceManager>& InResourceManager)
	: FSlateRenderer(InSlateFontServices)
	, ResourceManager(InResourceManager)
	, DrawBuffer(MakeUnique<FSlateDrawBuffer>())
{
}

bool FSlateNullRenderer::Initialize()
{
	return true;
}

void FSlateNullRenderer::Destroy()
{
	DrawBuffer.Reset();
}

FSlateDrawBuffer& FSlateNullRenderer::AcquireDrawBuffer()
{
	ensureMsgf(!DrawBuffer->IsLocked(), TEXT("The DrawBuffer is already locked. Make sure to call ReleaseDrawBuffer to release the DrawBuffer"));
	DrawBuffer->Lock();

	// Clear out the buffer each time its accessed
	DrawBuffer->ClearBuffer();

	return *DrawBuffer;
}

void FSlateNullRenderer::ReleaseDrawBuffer(FSlateDrawBuffer& InWindowDrawBuffer)
{
	ensureMsgf(DrawBuffer.Get() == &InWindowDrawBuffer, TEXT("It release a DrawBuffer that is not a member of the SlateNullRenderer"));
	InWindowDrawBuffer.Unlock();
}

void FSlateNullRenderer::CreateViewport( const TSharedRef<SWindow> Window )
{
}

void FSlateNullRenderer::UpdateFullscreenState( const TSharedRef<SWindow> Window, uint32 OverrideResX, uint32 OverrideResY )
{
}

void FSlateNullRenderer::RestoreSystemResolution(const TSharedRef<SWindow> InWindow)
{
}

void FSlateNullRenderer::OnWindowDestroyed( const TSharedRef<SWindow>& InWindow )
{
}

void FSlateNullRenderer::DrawWindows( FSlateDrawBuffer& WindowDrawBuffer )
{
}

FIntPoint FSlateNullRenderer::GenerateDynamicImageResource(const FName InTextureName)
{
	return FIntPoint( 0, 0 );
}

bool FSlateNullRenderer::GenerateDynamicImageResource( FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes )
{
	return false;
}

FSlateResourceHandle FSlateNullRenderer::GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	return ResourceManager.IsValid() ? ResourceManager->GetResourceHandle(Brush) : FSlateResourceHandle();
}

void FSlateNullRenderer::RemoveDynamicBrushResource( TSharedPtr<FSlateDynamicImageBrush> BrushToRemove )
{
}

void FSlateNullRenderer::ReleaseDynamicResource( const FSlateBrush& InBrush )
{
}

void FSlateNullRenderer::PrepareToTakeScreenshot(const FIntRect& Rect, TArray<FColor>* OutColorData, SWindow* InScreenshotWindow)
{
	if (OutColorData)
	{
		int32 TotalSize = Rect.Width() * Rect.Height();
		OutColorData->Empty(TotalSize);
		OutColorData->AddZeroed(TotalSize);
	}
}

FSlateUpdatableTexture* FSlateNullRenderer::CreateUpdatableTexture(uint32 Width, uint32 Height)
{
	return nullptr;
}

FSlateUpdatableTexture* FSlateNullRenderer::CreateSharedHandleTexture(void* SharedHandle)
{
	return nullptr;
}

void FSlateNullRenderer::ReleaseUpdatableTexture(FSlateUpdatableTexture* Texture)
{
}

void FSlateNullRenderer::RequestResize( const TSharedPtr<SWindow>& Window, uint32 NewWidth, uint32 NewHeight )
{
}

FCriticalSection* FSlateNullRenderer::GetResourceCriticalSection()
{
	return &ResourceCriticalSection;
}

void FSlateNullRenderer::RegisterCurrentScene_Impl(FSceneInterface* Scene)
{
	// This is a no-op
}

int32 FSlateNullRenderer::GetCurrentSceneIndex() const
{
	// This is a no-op
	return -1;
}

void FSlateNullRenderer::SetCurrentSceneIndex(int32 InIndex)
{
	// This is a no-op
}

void FSlateNullRenderer::ClearScenes() 
{
	// This is a no-op
}

void FSlateNullRenderer::Sync() const
{
#if UE_SLATE_NULL_RENDERER_WITH_ENGINE
	FFrameEndSync::Sync(FFrameEndSync::EFlushMode::EndFrame);
#endif
}
