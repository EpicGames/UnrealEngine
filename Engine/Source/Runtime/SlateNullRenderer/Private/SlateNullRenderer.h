// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateShaderResource.h"
#include "Rendering/SlateRenderer.h"

class FSlateDrawBuffer;
class FSlateFontServices;
class FSlateShaderResourceManager;
class FSlateUpdatableTexture;
class SWindow;
struct Rect;

/** A slate null rendering implementation */
class FSlateNullRenderer : public FSlateRenderer
{
public:
	FSlateNullRenderer(const TSharedRef<FSlateFontServices>& InSlateFontServices, const TSharedRef<FSlateShaderResourceManager>& InResourceManager);

	/** FSlateRenderer interface */
	virtual bool Initialize() override;
	virtual void Destroy() override;
	virtual FSlateDrawBuffer& AcquireDrawBuffer() override;
	virtual void ReleaseDrawBuffer(FSlateDrawBuffer& InWindowDrawBuffer) override;
	virtual void OnWindowDestroyed( const TSharedRef<SWindow>& InWindow ) override;
	virtual void RequestResize( const TSharedPtr<SWindow>& Window, uint32 NewWidth, uint32 NewHeight ) override;
	virtual void CreateViewport( const TSharedRef<SWindow> Window ) override;
	virtual void UpdateFullscreenState( const TSharedRef<SWindow> Window, uint32 OverrideResX, uint32 OverrideResY ) override;
	virtual void SetSystemResolution(uint32 Width, uint32 Height) override {}
	virtual void RestoreSystemResolution(const TSharedRef<SWindow> InWindow) override;
	virtual void DrawWindows( FSlateDrawBuffer& InWindowDrawBuffer ) override;
	virtual void ReleaseDynamicResource( const FSlateBrush& InBrush ) override;
	virtual void RemoveDynamicBrushResource( TSharedPtr<FSlateDynamicImageBrush> BrushToRemove ) override;
	virtual FIntPoint GenerateDynamicImageResource(const FName InTextureName) override;
	virtual bool GenerateDynamicImageResource( FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes ) override;
	virtual void PrepareToTakeScreenshot(const FIntRect& Rect, TArray<FColor>* OutColorData, SWindow* InScreenshotWindow) override;
	virtual FSlateResourceHandle GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale) override;
	virtual FSlateUpdatableTexture* CreateUpdatableTexture(uint32 Width, uint32 Height) override;
	virtual FSlateUpdatableTexture* CreateSharedHandleTexture(void* SharedHandle) override;
	virtual void ReleaseUpdatableTexture(FSlateUpdatableTexture* Texture) override;
	virtual FCriticalSection* GetResourceCriticalSection() override;
	virtual int32 GetCurrentSceneIndex() const override;
	virtual void SetCurrentSceneIndex(int32 InIndex) override;
	virtual void ClearScenes() override;
	virtual void Sync() const override;

protected:
	virtual void RegisterCurrentScene_Impl(FSceneInterface* Scene) override;

private:
	TSharedPtr<FSlateShaderResourceManager> ResourceManager;
	TUniquePtr<FSlateDrawBuffer> DrawBuffer;
	FCriticalSection ResourceCriticalSection;
};
