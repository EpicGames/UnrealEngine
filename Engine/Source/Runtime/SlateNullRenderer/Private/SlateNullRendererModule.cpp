// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateShaderResource.h"
#include "Textures/TextureAtlas.h"
#include "Rendering/ShaderResourceManager.h"
#include "Fonts/FontTypes.h"
#include "Fonts/FontCache.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateDrawBuffer.h"
#include "Rendering/SlateRenderer.h"
#if WITH_ENGINE
#include "TextureResource.h"
#endif
#include "Interfaces/ISlateNullRendererModule.h"
#include "SlateNullRenderer.h"

class FSlateNullShaderResourceManager : public ISlateAtlasProvider, public FSlateShaderResourceManager
{
public:
	// ISlateAtlasProvider interface
	virtual int32 GetNumAtlasPages() const override { return 0; }
	virtual class FSlateShaderResource* GetAtlasPageResource(const int32 InIndex) const override { return nullptr; }
	virtual bool IsAtlasPageResourceAlphaOnly(const int32 InIndex) const override { return false; }
#if WITH_ATLAS_DEBUGGING
	FAtlasSlotInfo GetAtlasSlotInfoAtPosition(FIntPoint InPosition, int32 AtlasIndex) const { return FAtlasSlotInfo(); }
#endif

	// FSlateShaderResourceManager interface
	virtual FSlateShaderResourceProxy* GetShaderResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale) override { return nullptr; }
	virtual ISlateAtlasProvider* GetTextureAtlasProvider() { return this; }
	virtual FSlateResourceHandle GetResourceHandle( const FSlateBrush& InBrush ) override 
	{
		static const FSlateResourceHandle NullHandle(MakeShareable(new FSlateSharedHandleData));
		return NullHandle;
	}
};

/** A null font texture resource to represent fonts */
class FSlateFontTextureNull : public FSlateShaderResource
#if WITH_ENGINE
	, public FTextureResource
#endif
{
public:
	FSlateFontTextureNull() {}

	/** FSlateShaderResource interface */
	virtual uint32 GetWidth() const override { return 0; }
	virtual uint32 GetHeight() const override { return 0; }
	virtual ESlateShaderResource::Type GetType() const override
	{
		return ESlateShaderResource::NativeTexture;
	}

#if WITH_ENGINE
	/** FTextureResource interface */
	virtual uint32 GetSizeX() const override { return 0; }
	virtual uint32 GetSizeY() const override { return 0; }
	virtual FString GetFriendlyName() const override { return TEXT("FSlateFontTextureNull"); }
#endif
};

/** A null font atlas store null font textures */
class FSlateFontAtlasNull : public FSlateFontAtlas
{
public:
	FSlateFontAtlasNull(float AtlasSize)
		: FSlateFontAtlas(AtlasSize, AtlasSize, ESlateFontAtlasContentType::Alpha, ESlateTextureAtlasPaddingStyle::PadWithZero)
	{}

	virtual class FSlateShaderResource* GetSlateTexture() const override { return &NullFontTexture; }
	virtual class FTextureResource* GetEngineTexture() override
	{
#if WITH_ENGINE
		return &NullFontTexture;
#else
		return nullptr;
#endif
	}
	virtual void ConditionalUpdateTexture()  override {}
	virtual void ReleaseResources() override {}

	static FSlateFontTextureNull NullFontTexture;
};
FSlateFontTextureNull FSlateFontAtlasNull::NullFontTexture;

/** A null font atlas factory to generate a null font atlas */
class FSlateNullFontAtlasFactory : public ISlateFontAtlasFactory
{
public:
	FSlateNullFontAtlasFactory()
		: AtlasSize(2048)
	{}

	virtual ~FSlateNullFontAtlasFactory() {}

	virtual FIntPoint GetAtlasSize(ESlateFontAtlasContentType InContentType) const override
	{
		return FIntPoint(AtlasSize, AtlasSize);
	}

	virtual TSharedRef<FSlateFontAtlas> CreateFontAtlas(ESlateFontAtlasContentType InContentType) const override
	{
		return MakeShareable(new FSlateFontAtlasNull(AtlasSize));
	}

	virtual TSharedPtr<ISlateFontTexture> CreateNonAtlasedTexture(const uint32 InWidth, const uint32 InHeight, ESlateFontAtlasContentType InContentType, const TArray<uint8>& InRawData) const override
	{
		return nullptr;
	}

private:
	/** Size of each font texture, width and height. Only used to return sane numbers */
	int32 AtlasSize;
};

/**
 * Implements the Slate Null Renderer module.
 */
class FSlateNullRendererModule
	: public ISlateNullRendererModule
{
public:

	// ISlateNullRendererModule interface
	virtual TSharedRef<FSlateRenderer> CreateSlateNullRenderer( ) override
	{
		ConditionalCreateResources();

		return MakeShareable( new FSlateNullRenderer(SlateFontServices.ToSharedRef(), ResourceManager.ToSharedRef()) );
	}

	virtual TSharedRef<ISlateFontAtlasFactory> CreateSlateFontAtlasFactory() override
	{
		return MakeShareable( new FSlateNullFontAtlasFactory );
	}

	virtual void StartupModule( ) override { }
	virtual void ShutdownModule( ) override { }

private:
	void ConditionalCreateResources()
	{
		if (!SlateFontServices.IsValid())
		{
			const TSharedRef<FSlateFontCache> FontCache = MakeShareable(new FSlateFontCache(MakeShareable(new FSlateNullFontAtlasFactory), ESlateTextureAtlasThreadId::Game));

			SlateFontServices = MakeShareable(new FSlateFontServices(FontCache, FontCache));
		}

		if (!ResourceManager.IsValid())
		{
			ResourceManager = MakeShareable(new FSlateNullShaderResourceManager);
		}
	}

private:
	TSharedPtr<FSlateFontServices> SlateFontServices;
	TSharedPtr<FSlateShaderResourceManager> ResourceManager;
};


IMPLEMENT_MODULE( FSlateNullRendererModule, SlateNullRenderer ) 
