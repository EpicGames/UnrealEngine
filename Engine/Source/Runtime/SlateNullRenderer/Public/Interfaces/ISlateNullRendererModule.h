// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Fonts/FontTypes.h"
#include "Rendering/SlateRenderer.h"

/**
 * Interface for the Slate RHI Renderer module.
 */
class ISlateNullRendererModule
	: public IModuleInterface
{
public:

	/**
	 * Creates a Slate RHI renderer.
	 *
	 * @return A new renderer which renders nothing.
	 */
	virtual TSharedRef<FSlateRenderer> CreateSlateNullRenderer( ) = 0;

	virtual TSharedRef<ISlateFontAtlasFactory> CreateSlateFontAtlasFactory() = 0;
};
