// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlateNullRenderer : ModuleRules
{
	public SlateNullRenderer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"SlateCore"
			}
		);

		if (Target.bCompileAgainstEngine)
		{
			PrivateDefinitions.Add("UE_SLATE_NULL_RENDERER_WITH_ENGINE=1");
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Engine",
					"RenderCore",
					"SlateBaseRenderer",
					"RHI"
				});
		}
		else
		{
			PrivateDefinitions.Add("UE_SLATE_NULL_RENDERER_WITH_ENGINE=0");
		}
	}
}
